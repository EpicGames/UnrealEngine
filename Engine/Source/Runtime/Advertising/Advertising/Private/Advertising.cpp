// Copyright Epic Games, Inc. All Rights Reserved.

#include "Advertising.h"
#include "Interfaces/IAdvertisingProvider.h"

IMPLEMENT_MODULE( FAdvertising, Advertising );

DEFINE_LOG_CATEGORY_STATIC( LogAdvertising, Display, All );

FAdvertising::FAdvertising()
{
}

FAdvertising::~FAdvertising()
{
}

IAdvertisingProvider * FAdvertising::GetAdvertisingProvider( const FName& ProviderName )
{
	// Check if we can successfully load the module.
	if ( ProviderName != NAME_None )
	{
		IAdvertisingProvider * Module = FModuleManager::Get().LoadModulePtr<IAdvertisingProvider>(ProviderName);
		if ( Module != NULL )
		{
			UE_LOGF(LogAdvertising, Log, "Creating Advertising provider %ls", *ProviderName.ToString());
			return Module;
		}
		else
		{
			UE_LOGF(LogAdvertising, Warning, "Failed to find Advertising provider named %ls.", *ProviderName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogAdvertising, Warning, "GetAdvertisingProvider called with a module name of None.");
	}
	return NULL;
}

void FAdvertising::StartupModule()
{
}

void FAdvertising::ShutdownModule()
{
}
