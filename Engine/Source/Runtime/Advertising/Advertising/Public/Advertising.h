// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"

class IAdvertisingProvider;

/**
 * Advertising module interface implementation
 */
class FAdvertising : public IModuleInterface
{
	//--------------------------------------------------------------------------
	// Module functionality
	//--------------------------------------------------------------------------
public:
	FAdvertising();
	virtual ~FAdvertising();

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FAdvertising& Get()
	{
		return FModuleManager::LoadModuleChecked< FAdvertising >( "Advertising" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "Advertising" );
	}

	static FName GetDefaultProviderName()
	{
		FString ProviderName;
		GConfig->GetString( TEXT( "Advertising" ), TEXT( "DefaultProviderName" ), ProviderName, GEngineIni );
		return FName( *ProviderName );
	}

	virtual IAdvertisingProvider* GetAdvertisingProvider( const FName& ProviderName );

	virtual IAdvertisingProvider* GetDefaultProvider()
	{
		return GetAdvertisingProvider( GetDefaultProviderName() );
	}

private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

