// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/** Generic interface for an advertising provider. Other modules can define more and register them with this module. */
class IAdvertisingProvider : public IModuleInterface
{
public:
	virtual void ShowAdBanner( bool bShowOnBottomOfScreen, int32 adID ) = 0;
	virtual void HideAdBanner() = 0;
	virtual void CloseAdBanner() = 0;
	virtual void LoadInterstitialAd(int32 adID) = 0;
	virtual bool IsInterstitialAdAvailable() = 0;
	virtual bool IsInterstitialAdRequested() = 0;
	virtual void ShowInterstitialAd() = 0;
	virtual int32 GetAdIDCount() = 0;
};

