// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class AndroidAdvertising : ModuleRules
	{
		public AndroidAdvertising( ReadOnlyTargetRules Target ) : base(Target)
		{			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					// ... add other public dependencies that you statically link with here ...
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Advertising",
                    // ... add private dependencies that you statically link with here ...
				}
				);
			PublicIncludePathModuleNames.Add("Advertising");

			if (Target.Platform == UnrealTargetPlatform.Android)
			{
				string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
				AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "AndroidAdvertising_APL.xml"));
			}
		}
	}
}
