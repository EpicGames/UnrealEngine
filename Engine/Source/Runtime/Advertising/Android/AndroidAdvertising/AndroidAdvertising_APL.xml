<?xml version="1.0" encoding="utf-8"?>
<!--AndroidAdvertising optional additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="AndroidAdvertising init"/>

		<!-- get packaging with ad support -->
		<setBoolFromProperty result="bRequestGAID" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="bRequestGAID" default="false"/>
		<setBoolFromProperty result="bSupportAdMob" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="bSupportAdMob" default="true"/>
		<setStringFromProperty result="AdMobAppID" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="AdMobAppID" default="[NOT SET]"/>

		<setStringFromProperty result="TagForChildDirectedTreatment" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="TagForChildDirectedTreatment" default="TAG_FOR_CHILD_DIRECTED_TREATMENT_UNSPECIFIED"/>
		<setStringFromProperty result="TagForUnderAgeOfConsent" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="TagForUnderAgeOfConsent" default="TAG_FOR_UNDER_AGE_OF_CONSENT_UNSPECIFIED"/>
		<setStringFromProperty result="MaxAdContentRating" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="MaxAdContentRating" default="MAX_AD_CONTENT_RATING_PG"/>

		<if condition="bSupportAdMob">
			<true>
				<setBoolIsEqual result="bAdMobAppIDNotSet" arg1="$S(AdMobAppID)" arg2="[NOT SET]"/>
				<setBoolIsEqual result="bAdMobAppIDEmpty" arg1="$S(AdMobAppID)" arg2=""/>
				<setBoolOr result="bAdMobAppIDInvalid" arg1="$B(bAdMobAppIDNotSet)" arg2="$B(bAdMobAppIDEmpty)"/>
				<if condition="bAdMobAppIDInvalid">
					<true>
						<log text="AdMob App ID is not set; disabling AdMob support!"/>
						<setBool result="bSupportAdMob" value="false"/>
					</true>
				</if>
			</true>
		</if>
		
		<if condition="bSupportAdMob">
			<true>
				<log text="Including AdMob support. Disable in Android project settings if you do not need to use it."/>
			</true>
			<false>
				<log text="Not including AdMob support. Enable in Android project settings if you need to use it."/>
				<setBool result="bRequestGAID" value="false"/>
			</false>
		</if>
	</init>

	<buildGradleAdditions>
		<if condition="bSupportAdMob">
			<true>
				<insert>
dependencies {
	implementation('com.google.android.gms:play-services-ads:18.1.0')
	implementation('com.google.android.gms:play-services-ads-identifier:18.0.1')
}
				</insert>
			</true>
		</if>
	</buildGradleAdditions>

	<!-- optional updates applied to AndroidManifest.xml -->
	<androidManifestUpdates>
		<if condition="bSupportAdMob">
			<true>
				<addPermission android:name="android.permission.INTERNET"/>
				<addPermission android:name="android.permission.ACCESS_NETWORK_STATE"/>
				<setElement result="AdMobMeta" value="meta-data" />
				<addAttribute tag="$AdMobMeta" name="android:name" value="com.google.android.gms.ads.APPLICATION_ID" />
				<addAttribute tag="$AdMobMeta" name="android:value" value="$S(AdMobAppID)" />
				<addElement tag="application" name="AdMobMeta" />
			</true>
		</if>
	</androidManifestUpdates>

	<!-- optional additions to the GameActivity imports in GameActivity.java -->
	<gameActivityImportAdditions>
		<if condition="bSupportAdMob">
			<true>
				<insert>
import com.google.android.gms.common.GoogleApiAvailability;
import com.google.android.gms.ads.MobileAds;
import com.google.android.gms.ads.initialization.InitializationStatus;
import com.google.android.gms.ads.initialization.OnInitializationCompleteListener;
import com.google.android.gms.ads.AdRequest;
import com.google.android.gms.ads.AdView;
import com.google.android.gms.ads.AdSize;
import com.google.android.gms.ads.AdListener;
import com.google.android.gms.ads.InterstitialAd;
import com.google.android.gms.ads.RequestConfiguration;
import com.google.android.gms.ads.identifier.AdvertisingIdClient;
import com.google.android.gms.ads.identifier.AdvertisingIdClient.Info;
				</insert>
			</true>
		</if>
	</gameActivityImportAdditions>

	<!-- optional additions to the GameActivity class in GameActivity.java -->
	<gameActivityClassAdditions>
		<if condition="bSupportAdMob">
			<true>
				<insert>
<![CDATA[
    /** AdMob support */
    private PopupWindow adPopupWindow;
    private AdView adView;
    private boolean adInit = false;
    private LinearLayout adLayout;
    private int adGravity = Gravity.TOP;
    private InterstitialAd interstitialAd;
    private boolean isInterstitialAdLoaded = false;
    private boolean isInterstitialAdRequested = false;
    private AdRequest interstitialAdRequest;
    private String advertisingID = null;

	/** true when the application has requested that an ad be displayed */
	private boolean adWantsToBeShown = false;

	/** true when an ad is available to be displayed */
	private boolean adIsAvailable = false;

	/** true when an ad request is in flight */
	private boolean adIsRequested = false;

	// handle ad popup visibility and requests
	private void updateAdVisibility(boolean loadIfNeeded)
	{
		if (!adInit || (adPopupWindow == null))
		{
			return;
		}

		// request an ad if we don't have one available or requested, but would like one
		if (adWantsToBeShown && !adIsAvailable && !adIsRequested && loadIfNeeded)
		{
			AdRequest adRequest = new AdRequest.Builder().build();		// add test devices here
			GameActivity.Get().adView.loadAd(adRequest);

			adIsRequested = true;
		}

		if (adIsAvailable && adWantsToBeShown)
		{
			if (adPopupWindow.isShowing())
			{
				return;
			}

			adPopupWindow.showAtLocation(activityLayout, adGravity, 0, 0);
			// don't call update on 7.0 to work around this issue: https://code.google.com/p/android/issues/detail?id=221001
			if (ANDROID_BUILD_VERSION != 24) {
				adPopupWindow.update();
			}
		}
		else
		{
			if (!adPopupWindow.isShowing())
			{
				return;
			}

			adPopupWindow.dismiss();
			adPopupWindow.update();
		}
	}

	public void AndroidThunkJava_ShowAdBanner(String AdMobAdUnitID, boolean bShowOnBottonOfScreen)
	{
		Log.debug("In AndroidThunkJava_ShowAdBanner");
		Log.debug("AdID: " + AdMobAdUnitID);

		adGravity = bShowOnBottonOfScreen ? Gravity.BOTTOM : Gravity.TOP;

		if (adInit)
		{
			// already created, make it visible
			activityContext.runOnUiThread(new Runnable()
			{
				@Override
				public void run()
				{
					if ((adPopupWindow == null) || adPopupWindow.isShowing())
					{
						return;
					}

					adWantsToBeShown = true;
					updateAdVisibility(true);
				}
			});

			return;
		}

		// init our AdMob window
		adView = new AdView(this);
		adView.setAdUnitId(AdMobAdUnitID);
		adView.setAdSize(AdSize.BANNER);

		if (adView != null)
		{
			activityContext.runOnUiThread(new Runnable()
			{
				@Override
				public void run()
				{
					adInit = true;

					final DisplayMetrics dm = getResources().getDisplayMetrics();
					final float scale = dm.density;
					adPopupWindow = new PopupWindow(_activity);
					adPopupWindow.setWidth((int)(320*scale));
					adPopupWindow.setHeight((int)(50*scale));
					adPopupWindow.setClippingEnabled(false);

					adLayout = new LinearLayout(_activity);

					final int padding = (int)(-5*scale);
					adLayout.setPadding(padding,padding,padding,padding);

					MarginLayoutParams params = new MarginLayoutParams(LayoutParams.WRAP_CONTENT, LayoutParams.WRAP_CONTENT);;

					params.setMargins(0,0,0,0);

					adLayout.setOrientation(LinearLayout.VERTICAL);
					adLayout.addView(adView, params);
					adPopupWindow.setContentView(adLayout);

					// set up our ad callbacks
					GameActivity.Get().adView.setAdListener(new AdListener()
					{
						 @Override
						public void onAdLoaded()
						{
							adIsAvailable = true;
							adIsRequested = false;

							updateAdVisibility(true);
						}

						 @Override
						public void onAdFailedToLoad(int errorCode)
						{
							adIsAvailable = false;
							adIsRequested = false;

							// don't immediately request a new ad on failure, wait until the next show
							updateAdVisibility(false);
						}
					});

					adWantsToBeShown = true;
					updateAdVisibility(true);
				}
			});
		}
	}

	public void AndroidThunkJava_HideAdBanner()
	{
		Log.debug("In AndroidThunkJava_HideAdBanner");

		if (!adInit)
		{
			return;
		}

		activityContext.runOnUiThread(new Runnable()
		{
			@Override
			public void run()
			{
				adWantsToBeShown = false;
				updateAdVisibility(true);
			}
		});
	}

	public void AndroidThunkJava_CloseAdBanner()
	{
		Log.debug("In AndroidThunkJava_CloseAdBanner");

		if (!adInit)
		{
			return;
		}

		// currently the same as hide.  should we do a full teardown?
		activityContext.runOnUiThread(new Runnable()
		{
			@Override
			public void run()
			{
				adWantsToBeShown = false;
				updateAdVisibility(true);
			}
		});
	}
	
	public void AndroidThunkJava_LoadInterstitialAd(String AdMobAdUnitID)
	{
		interstitialAdRequest = new AdRequest.Builder().build();

		interstitialAd = new InterstitialAd(this);
		isInterstitialAdLoaded = false;
		isInterstitialAdRequested = true;
		interstitialAd.setAdUnitId(AdMobAdUnitID);

		activityContext.runOnUiThread(new Runnable()
		{
			@Override
			public void run()
			{
				interstitialAd.loadAd(interstitialAdRequest);				
			}
		});
		
		interstitialAd.setAdListener(new AdListener()
		{
			@Override
			public void onAdFailedToLoad(int errorCode) 
			{
				Log.debug("Interstitial Ad failed to load, errocode: " + errorCode);
				isInterstitialAdLoaded = false;
				isInterstitialAdRequested = false;
			}
			@Override
			public void onAdLoaded() 
			{
				//track if the ad is loaded since we can only called interstitialAd.isLoaded() from the uiThread				
				isInterstitialAdLoaded = true;
				isInterstitialAdRequested = false;
			}    
		});
	}

	public boolean AndroidThunkJava_IsInterstitialAdAvailable()
	{
		return interstitialAd != null && isInterstitialAdLoaded;
	}

	public boolean AndroidThunkJava_IsInterstitialAdRequested()
	{
		return interstitialAd != null && isInterstitialAdRequested;
	}

	public void AndroidThunkJava_ShowInterstitialAd()
	{
		if(isInterstitialAdLoaded)
		{
			activityContext.runOnUiThread(new Runnable()
			{
				@Override
				public void run()
				{					
					interstitialAd.show();
				}
			});
		}
		else
		{
			Log.debug("Interstitial Ad is not available to show - call LoadInterstitialAd or wait for it to finish loading");
		}
	}

	private GetAdvertisingIdTask AdTask = null;

	private class GetAdvertisingIdTask extends android.os.AsyncTask<String, Integer, String>
	{
		@Override
		protected String doInBackground(String... values)
		{
			AdvertisingIdClient.Info adInfo = null;
			try
			{
				adInfo = AdvertisingIdClient.getAdvertisingIdInfo(GameActivity.Get().getApplicationContext());
				if (adInfo.isLimitAdTrackingEnabled())
				{
					Log.debug("GetAdvertisingId: User opted out of ad tracking");
					adInfo = null;
				}
				Log.debug("GetAdvertisingID: success");
			}
			catch (Exception e) {
				Log.debug("GetAdvertisingId failed: " + e.getMessage());
			}
			return (adInfo == null) ? "" : adInfo.getId();
		}
		
		@Override
		protected void onPostExecute(String s)
		{
			advertisingID = s;
		}
	}

	public String AndroidThunkJava_GetAdvertisingId()
	{
		try
		{
			AdTask.get();
		}
		catch (Exception e)
		{
			advertisingID = null;
		}

		return advertisingID;
	}
]]>
				</insert>
			</true>
			<false>
				<insert>
	public void AndroidThunkJava_ShowAdBanner(String AdMobAdUnitID, boolean bShowOnBottonOfScreen)
	{
	}

	public void AndroidThunkJava_HideAdBanner()
	{
	}

	public void AndroidThunkJava_CloseAdBanner()
	{
	}

	public void AndroidThunkJava_LoadInterstitialAd(String AdMobAdUnitID)
	{
	}

	public boolean AndroidThunkJava_IsInterstitialAdAvailable()
	{
		return true;
	}

	public boolean AndroidThunkJava_IsInterstitialAdRequested()
	{
		return false;
	}

	public void AndroidThunkJava_ShowInterstitialAd()
	{
	}

	public String AndroidThunkJava_GetAdvertisingId()
	{
		return null;
	}
				</insert>
			</false>
		</if>
	</gameActivityClassAdditions>
	
	<gameActivityOnCreateAdditions>
		<if condition="bSupportAdMob">
			<true>
				<insert>
		RequestConfiguration AdMobConfig = new RequestConfiguration.Builder()
				</insert>
				<insertValue value="				.setTagForChildDirectedTreatment(RequestConfiguration.$S(TagForChildDirectedTreatment))"/>
				<insertNewline/>
				<insertValue value="				.setTagForUnderAgeOfConsent(RequestConfiguration.$S(TagForUnderAgeOfConsent))"/>
				<insertNewline/>
				<insertValue value="				.setMaxAdContentRating(RequestConfiguration.$S(MaxAdContentRating))"/>
				<insertNewline/>
				<insert>
				.build();
		MobileAds.setRequestConfiguration(AdMobConfig);
        MobileAds.initialize(this, new OnInitializationCompleteListener() {
            @Override
            public void onInitializationComplete(InitializationStatus initializationStatus) {
            }
        });
				</insert>
			</true>
		</if>
	</gameActivityOnCreateAdditions>

	<gameActivityBeforeConfigRulesAppliedAdditions>
		<if condition="bRequestGAID">
			<true>
				<insert>
			try
			{
				if (GoogleApiAvailability.getInstance().isGooglePlayServicesAvailable(this) == 0)
				{
					AdTask = new GetAdvertisingIdTask();
					AdTask.execute();
				}
			}
			catch (Exception e)
			{
			}
				</insert>
			</true>
		</if>
	</gameActivityBeforeConfigRulesAppliedAdditions>

</root>
