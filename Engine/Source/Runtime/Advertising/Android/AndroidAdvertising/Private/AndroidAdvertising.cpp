// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidAdvertising.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"


DEFINE_LOG_CATEGORY_STATIC( LogAdvertising, Display, All );

IMPLEMENT_MODULE( FAndroidAdvertisingProvider, AndroidAdvertising );


void FAndroidAdvertisingProvider::ShowAdBanner(bool bShowOnBottomOfScreen, int32 adID)
{
	extern void AndroidThunkCpp_ShowAdBanner(const FString&, bool);

	TArray<FString> AdUnitIDs;
	int32 count = GConfig->GetArray(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("AdMobAdUnitIDs"), AdUnitIDs, GEngineIni);

	if (count == 0)
	{
		// Fall back to checking old setting
		FString AdUnitID;
		bool found = GConfig->GetString(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("AdMobAdUnitID"), AdUnitID, GEngineIni);
		if (found && !AdUnitID.IsEmpty())
			AndroidThunkCpp_ShowAdBanner(AdUnitID, bShowOnBottomOfScreen);
		return;
	}

	if (adID >= 0 && adID < count && !AdUnitIDs[adID].IsEmpty())
		AndroidThunkCpp_ShowAdBanner(AdUnitIDs[adID], bShowOnBottomOfScreen);
}

void FAndroidAdvertisingProvider::HideAdBanner() 
{
	extern void AndroidThunkCpp_HideAdBanner();
	AndroidThunkCpp_HideAdBanner();
}

void FAndroidAdvertisingProvider::CloseAdBanner() 
{
	extern void AndroidThunkCpp_CloseAdBanner();
	AndroidThunkCpp_CloseAdBanner();
}

int32 FAndroidAdvertisingProvider::GetAdIDCount()
{
	TArray<FString> AdUnitIDs;
	int32 count = GConfig->GetArray(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("AdMobAdUnitIDs"), AdUnitIDs, GEngineIni);

	return count;
}

void FAndroidAdvertisingProvider::LoadInterstitialAd(int32 adID)
{
	extern void AndroidThunkCpp_LoadInterstitialAd(const FString&);

	TArray<FString> AdUnitIDs;
	int32 count = GConfig->GetArray(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("AdMobAdUnitIDs"), AdUnitIDs, GEngineIni);

	if (count == 0)
	{
		// Fall back to checking old setting
		FString AdUnitID;
		bool found = GConfig->GetString(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("AdMobAdUnitID"), AdUnitID, GEngineIni);
		if (found && !AdUnitID.IsEmpty())
			AndroidThunkCpp_LoadInterstitialAd(AdUnitID);
		return;
	}

	if (adID >= 0 && adID < count && !AdUnitIDs[adID].IsEmpty())
	{
		AndroidThunkCpp_LoadInterstitialAd(AdUnitIDs[adID]);
	}
}

bool FAndroidAdvertisingProvider::IsInterstitialAdAvailable()
{
	extern bool AndroidThunkCpp_IsInterstitialAdAvailable();

	return AndroidThunkCpp_IsInterstitialAdAvailable();
}

bool FAndroidAdvertisingProvider::IsInterstitialAdRequested()
{
	extern bool AndroidThunkCpp_IsInterstitialAdRequested();

	return AndroidThunkCpp_IsInterstitialAdRequested();
}

void FAndroidAdvertisingProvider::ShowInterstitialAd()
{
	extern void AndroidThunkCpp_ShowInterstitialAd();

	AndroidThunkCpp_ShowInterstitialAd();
}