// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IAdvertisingProvider.h"

class FAndroidAdvertisingProvider : public IAdvertisingProvider
{
public:
	virtual void ShowAdBanner(bool bShowOnBottomOfScreen, int32 adID) override;
	virtual void HideAdBanner() override;
	virtual void CloseAdBanner() override;

	virtual void LoadInterstitialAd(int32 adID) override;
	virtual bool IsInterstitialAdAvailable() override;
	virtual bool IsInterstitialAdRequested() override;
	virtual void ShowInterstitialAd() override;

	virtual int32 GetAdIDCount() override;
};
