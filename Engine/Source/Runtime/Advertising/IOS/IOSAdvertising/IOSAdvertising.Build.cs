// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class IOSAdvertising : ModuleRules
	{
		public IOSAdvertising( ReadOnlyTargetRules Target ) : base(Target)
		{
			bRequiresPlatformSDK = true;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					// ... add other public dependencies that you statically link with here ...
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Advertising",
					"ApplicationCore"
                    // ... add private dependencies that you statically link with here ...
				}
				);
			PublicIncludePathModuleNames.Add("Advertising");
		}
	}
}
