// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSAdvertising.h"
#include "IOS/IOSAppDelegate.h"
#include "Modules/ModuleManager.h"
#include "IOS/IOSAsyncTask.h"
#include "IOS/IOSAppDelegate.h"

DEFINE_LOG_CATEGORY_STATIC( LogAdvertising, Display, All );

IMPLEMENT_MODULE( FIOSAdvertisingProvider, IOSAdvertising );
