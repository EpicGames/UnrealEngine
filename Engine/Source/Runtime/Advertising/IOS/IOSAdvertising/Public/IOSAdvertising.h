// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IAdvertisingProvider.h"

class FIOSAdvertisingProvider : public IAdvertisingProvider
{
public:
    //empty functions for now, this is Android-only support until iAd is replaced by AdMob
    virtual void ShowAdBanner( bool bShowOnBottomOfScreen, int32 AdID ) override {}
    virtual void HideAdBanner() override {}
    virtual void CloseAdBanner() override {}
    virtual int32 GetAdIDCount() override { return -1; }
    virtual void LoadInterstitialAd(int32 adID) override {}
	virtual bool IsInterstitialAdAvailable() override { return false; }
	virtual bool IsInterstitialAdRequested() override { return false; }
	virtual void ShowInterstitialAd() override {}
};
